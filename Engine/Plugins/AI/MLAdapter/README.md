# MLAdapter
###### please note that this plugin is very much experimental and work-in-progress

The goal of the MLAdapter plugin is to supply external clients (like a python script) with an ability to interface with a running UE5 instance in an organized fashion.

A client can connect to an UE5 instance and using RPCs (remote procedure calls) add in-game agents, configure them (declaring what kind of Sensors and Actuators the agent needs), get data from the world (collected by agent’s sensors) and affect the world (via agent’s actuators). 

An optional but a very important part of the MLAdapter plugin is the accompanying python package (uemladapter) that adheres to OpenAI Gym’s API making it easy to work with MLAdapter-plugin-empowered-UE5-games/projects just like with any other OpenAI Gym environment. The python package makes such a UE5 game/project a regular OpenAI Gym environment (http://gym.openai.com/docs/) which means users can interact with a UE5 instance without changing their pipelines/workflows. At the moment the python code has no examples of agent training - right now we’re just supplying new environments to interact with.

The following diagram presents an overview of MLAdapter's architecture.

![MLAdapter diagram](Docs/img/MLAdapter_diagram.png)

Please note that the plugin is still in very early development. All feedback is highly encouraged. 

## Installation

### C++

*UE5 note: ActionRPG has not been updated to UE5 yet, but UE4 version compiles just fine*

The plugin’s source code can be found in Engine/Plugins/AI/MLAdapter. We also strongly suggest getting the latest `Samples/Games/ActionRPG`, as well as the last possible (recently removed) `PlatformerGame` version, as we've made some minor modifications to those samples making them cooperate better with the plugin. By default the plugin is not enabled for those games. To enable it add the following section to game’s *.uproject, right after the “Modules” section:

```
"Plugins": [		
	{
		"Name": "MLAdapter",
		"Enabled": true
	}
],
```

Note that it’s not necessary to regenerate project files after this change, the UBT will pick the change up automatically.

### Python

#### Getting python

First you need python on your machine. The absolute easiest way to get it is to download an installer from https://www.python.org/downloads/ (3.7.x would be best) and install it with default settings. If asked, agree to adding python to environment variables.

#### unreal.mladapter package
The unreal.mladapter python package can be found in `Engine/Plugins/AI/MLAdapter/Source/python`. To add the package to your python distribution just call the following in the `MLAdapter/Source/python` directory:

```
pip install -e .
```
The installation script will install the package's dependencies but if it turns out something’s missing please let me know.

## Running
On the C++ side all one needs to do is to enable the plugin for the given project and compile (to ensure MLAdapter plugin’s binaries are up to date).

On the python side, the unreal.mladapter package supports both connecting to a running instance of UE5 as well as launching one. Launching does require the user to add UE-DevBinaries to environment variables (pointing at the **directory** where the executable can be found, `UnrealEditor-Win64-Debug.exe` or `UnrealEditor.exe` will be used, depending on `unreal.mladapter.runner:_DEBUG`) or adding the `--exec` parameter to python script’s execution. The --exec should point at a specific executable **file** to use.

Example:

![img](Docs/img/MLAdapter_env_var.png)

or: 

```
python custom_params.py --exec=d:/p4/df/Engine/Binaries/Win64/UnrealEditor.exe
```
where `custom_params.py` is a script using the `unreal.mladapter` package. Both ways will result in the same UE5 binary being used.

## Example scripts
The `MLAdapter/Source/python/examples` directory contains example scripts one can use to construct/run/connect to a running UE5 game.

- `as_gym_env.py` - uses OpenAI Gym’s `gym.make` to construct an environment. 
- `custom_params.py` - hand-creates an unreal.mladapter environment, configures it and runs (including launching the UE5 instance)
- `connect_to_running.py` - connects to a running instance of an UE5 game/project.

So assuming you have your UE-DevBinaries environment variable set up you can just navigate to the examples directory and run the following command:
```
python custom_params.py
```
When executed this script will launch ActionRPG (make sure you have it synced!), connect to it, do a one playthrough (executing random actions) and close both the script and the UE5 instance as soon as one playthrough is done.

`unreal/mladapter/envs/__init__.py` contains a list of available environments. 

## unit tests
The python `unreal.mladapter` package comes with a suite of basic unit test. You can find the test scripts in `MLAdapter/Source/python/tests`. These scripts can be run individually just like any other python script, or run:
```
python -m unittest discover tests/
```
while in `MLAdapter/Source/python`.

## Current limitations
There’s a lot more, but here are some highlights:

- The camera’s sensor implementation is very naïve (which affects perf). Proper implementation pending.
- only the Windows platform is supported at the moment and we've tested it only on Win64. Note that the rpclib does support other platforms, we "just" need to compile rpclib for those platforms and it should work. 

## Practical advice
#### General
If your client doesn’t seem to be able to connect to the rpc server try using a different port. It’s the `-mladapterport=` option when launching UE5 instance and `server_port` parameter of unreal.mladapter environment’s constructor.

#### Python
When manually creating unreal.mladapter environments or connecting to one you want to debug on the C++ side it’s useful to add timeout parameter to environment’s constructor, like so:
```
env = ActionRPG(timeout=3600)
```
This will make sure the rpcclient won’t disconnect while you debug the C++ side (well, it will, after an hour!). The default timeout is 120 seconds and it might not be enough for bigger levels to load, especially when run in debug mode.

##### UnableToReachRPCServer
If you're getting `unreal.mladapter.error.UnableToReachRPCServer` thrown in your face check the `timeout` value (the one mentioned above). Try setting it to something huge and see if you're still getting the error when your UE5 instance finishes loading.

An alternative reason may be that your project doesn't have the `MLAdapter` plugin enabled. Make sure that your project does have 

```
{
	"Name": "MLAdapter",
	"Enabled": true
}
```

in it's `*.uproject` file's `"Plugins"` section.

##### WSAECONNREFUSED
You'll probably see repeaded `WSAECONNREFUSED` warning reported while the UE5 instance is launching. This is normal and nothing to worry about, the connection cannot be established until the MLAdapter plugin gets loaded by the UE5 instance and MLAdapterManager gets created, which can take some time. 
If you get the warning when the UE5 instance is fully up and running make sure the project has MLAdapter plugin enabled and it's listening on the port you're expecting. To force a specific port you can use `-MLAdapterPort=XXXXX` command line parameter for the UE5 instance and `--port=XXXXX` for the python script. 

## Feedback
Please send your feedback to ml-adapter@epicgames.com